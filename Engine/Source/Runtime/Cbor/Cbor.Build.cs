// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class Cbor : ModuleRules
	{
		public Cbor(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
				}
			);
			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
		}
    }
}
