// Copyright Epic Games, Inc. All Rights Reserved.

#include "CborReader.h"

#include "Math/Float16.h"

FCborReader::FCborReader(FArchive* InStream, ECborEndianness InReaderEndianness)
	: Stream(InStream)
	, Endianness(InReaderEndianness)
{
	check(InStream != nullptr);
	ContextStack.Emplace();
}

FCborReader::~FCborReader()
{
	check(ContextStack.Num() > 0 && (ContextStack[0].IsDummy() || ContextStack[0].IsError()));
}

const FArchive* FCborReader::GetArchive() const
{
	return Stream;
}

bool FCborReader::IsError() const
{
	// the dummy context holds previous error
	return ContextStack[0].IsError();
}

FCborHeader FCborReader::GetError() const
{
	// the dummy context holds previous error
	return ContextStack[0].Header;
}

const FCborContext& FCborReader::GetContext() const
{
	return ContextStack.Top();
}

bool FCborReader::ReadNext(FCborContext& OutContext)
{
	OutContext.Reset();

	// if an error happened, successive read are also errors 
	if (IsError())
	{
		OutContext.Header = GetError();
		return false;
	}

	// Invalid stream error
	if (Stream == nullptr)
	{
		OutContext.Header = SetError(ECborCode::ErrorStreamFailure);
		return false;
	}

	// Set accurate endianness handling
	ScopedCborArchiveEndianness ScopedArchiveEndianness(*Stream, Endianness);

	// Current parent
	FCborContext& ParentContext = ContextStack.Top();

	// Check if we reached container end, if so output as if we read a break code
	if (ParentContext.IsFiniteContainer() && ParentContext.Length == 0)
	{
		OutContext.Header.Set(ECborCode::Break);
		// Report 0 Length
		OutContext.Length = ParentContext.Length;
		// Report parent context container type
		OutContext.RawTextValue.Add((char)ParentContext.MajorType());
		// Done with parent context
		check(ContextStack.Num() > 1);
		ContextStack.Pop();
		return true;
	}

	// Done reading
	if (Stream->AtEnd())
	{
		OutContext.Header = (ParentContext.RawCode() == ECborCode::Dummy) ? FCborHeader(ECborCode::StreamEnd) : SetError(ECborCode::ErrorContext);
		return false;
	}

	// Read the cbor header
	*Stream << OutContext.Header;

	// Check for break item
	if (OutContext.IsBreak())
	{
		// Got a break item out of a indefinite context
		if (!ParentContext.IsIndefiniteContainer())
		{
			OutContext.Header = SetError(ECborCode::ErrorBreak);
			return false;
		}
		
		// Odd number of item read
		if (ParentContext.MajorType() == ECborCode::Map && (ParentContext.Length & 1))
		{
			OutContext.Header = SetError(ECborCode::ErrorMapContainer);
			return false;
		}
		// Report Length
		OutContext.Length = ParentContext.Length;
		// Report parent context container type
		OutContext.RawTextValue.Add((char)ParentContext.MajorType());
		// Done with parent context
		check(ContextStack.Num() > 1);
		ContextStack.Pop();
		return true;
	}

	// if the type is indefinite, we increment the length of the parent context
	if (ParentContext.IsIndefiniteContainer())
	{
		++ParentContext.Length;

		// If we have an indefinite string but current context type doesn't match flag an error
		if (ParentContext.IsString() && ParentContext.MajorType() != OutContext.MajorType())
		{
			OutContext.Header = SetError(ECborCode::ErrorStringNesting);
			return false;
		}
	}
	// Otherwise the length was set when we read the parent context, decrement it, container end if flagged when reaching 0
	else if (ParentContext.IsFiniteContainer())
	{
		--ParentContext.Length;
	}

	// Read item
	switch (OutContext.MajorType())
	{
		case ECborCode::Uint:
			OutContext.UIntValue = ReadUIntValue(OutContext, *Stream);
			break;
		case ECborCode::Int:
			OutContext.UIntValue = ~ReadUIntValue(OutContext, *Stream);
			break;
		case ECborCode::ByteString:
			// fall through
		case ECborCode::TextString:
			// if we have an indefinite string item, push the context
			if (OutContext.IsIndefiniteContainer())
			{
				OutContext.Length = 0;
				ContextStack.Push(OutContext);
			}
			// Otherwise read the string length in bytes, then serialize the raw context in the byte array
			else
			{
				OutContext.Length = ReadUIntValue(OutContext, *Stream);
				if (OutContext.Length > (uint64)(MAX_int32 - 1))
				{
					OutContext.Header = SetError(ECborCode::ErrorStringLength);
					return false;
				}
				int32 StringLength = (int32)OutContext.Length;
				OutContext.RawTextValue.SetNumUninitialized(StringLength + 1); // Length doesn't count the null terminating character
				Stream->Serialize(OutContext.RawTextValue.GetData(), OutContext.Length);
				OutContext.RawTextValue[StringLength] = '\0';
			}
			break;
		case ECborCode::Array:
			OutContext.Length = OutContext.AdditionalValue() == ECborCode::Indefinite ? 0 : ReadUIntValue(OutContext, *Stream);
			ContextStack.Push(OutContext);
			break;
		case ECborCode::Map:
			OutContext.Length = OutContext.AdditionalValue() == ECborCode::Indefinite ? 0 : ReadUIntValue(OutContext, *Stream) * 2;
			ContextStack.Push(OutContext);
			break;
		case ECborCode::Tag:
			OutContext.UIntValue = ReadUIntValue(OutContext, *Stream);
			break;
		case ECborCode::Prim:
			ReadPrimValue(OutContext, *Stream);
			break;
	}
	
	if (OutContext.IsError())
	{
		SetError(OutContext.RawCode());
		return false;
	}
	return true;
}

bool FCborReader::SkipContainer(ECborCode ContainerType)
{
	// Invalid stream error
	if (Stream == nullptr)
	{
		return false;
	}

	if (GetContext().MajorType() != ContainerType)
	{
		return false;
	}
	uint32 Depth = 0;
	FCborContext Context;
	while (ReadNext(Context))
	{
		if (Context.IsBreak() && Depth-- == 0)
		{
			break;
		}

		if (Context.IsContainer())
		{
			++Depth;
		}
	}
	return !IsError();
}

uint64 FCborReader::ReadUIntValue(FCborContext& Context, FArchive& Ar)
{
	uint64 AdditionalValue = (uint8)Context.AdditionalValue();
	switch (Context.AdditionalValue())
	{
	case ECborCode::Value_1Byte:
		{
			uint8 Temp;
			Ar << Temp;
			AdditionalValue = Temp;
		}
		break;
	case ECborCode::Value_2Bytes:
		{
			uint16 Temp;
			Ar << Temp;
			AdditionalValue = Temp;
		}
		break;
	case ECborCode::Value_4Bytes:
		{
			uint32 Temp;
			Ar << Temp;
			AdditionalValue = Temp;
		}
		break;
	case ECborCode::Value_8Bytes:
		{
			uint64 Temp;
			Ar << Temp;
			AdditionalValue = Temp;
		}
		break;
	case ECborCode::Unused_28:
		// Fall through
	case ECborCode::Unused_29:
		// Fall through
	case ECborCode::Unused_30:
		// Fall through
	case ECborCode::Indefinite:
		// Error
		Context.Header.Set(ECborCode::ErrorReservedItem);
		break;
	default:
		// Use value directly, Noop
		break;
	}
	return AdditionalValue;
}

void FCborReader::ReadPrimValue(FCborContext& Context, FArchive& Ar)
{
	switch (Context.AdditionalValue())
	{
	case ECborCode::False:
		Context.BoolValue = false;
		break;
	case ECborCode::True:
		Context.BoolValue = true;
		break;
	case ECborCode::Null:
		// fall through
	case ECborCode::Undefined:
		// noop
		break;
	case ECborCode::Value_1Byte:
		{
			uint8 Temp;
			Ar << Temp;
		}
		break;
	case ECborCode::Value_2Bytes:
		{
			FFloat16 Temp;
			Ar << Temp;
			// Use built in FFloat16 converter to assign to our float type. 
			Context.FloatValue = Temp;
		}
		break;
	case ECborCode::Value_4Bytes:
		{
			float Temp;
			Ar << Temp;
			Context.FloatValue = Temp;
		}
		break;
	case ECborCode::Value_8Bytes:
		{	
			double Temp;
			Ar << Temp;
			Context.DoubleValue = Temp;
		}
		break;
	default:
		// Error other values are unused, break item should have been processed elsewhere
		Context.Header.Set(ECborCode::ErrorReservedItem);
		break;
	}
}

FCborHeader FCborReader::SetError(ECborCode ErrorCode)
{
	FCborContext& Dummy = ContextStack[0];
	Dummy.Header.Set(ErrorCode);
	return Dummy.Header;
}
