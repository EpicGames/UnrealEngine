// Copyright Epic Games, Inc. All Rights Reserved.

#include "CborWriter.h"
#include "Containers/AnsiString.h"
#include "Containers/Utf8String.h"

FCborWriter::FCborWriter(FArchive* InStream, ECborEndianness InWriterEndianness)
	: Stream(InStream)
	, Endianness(InWriterEndianness)
{
	check(Stream != nullptr && Stream->IsSaving());
	ContextStack.Emplace();
}

FCborWriter::~FCborWriter()
{
	check(ContextStack.Num() == 1 && ContextStack.Top().RawCode() == ECborCode::Dummy);
}

const FArchive* FCborWriter::GetArchive() const
{
	return Stream;
}

void FCborWriter::WriteContainerStart(ECborCode ContainerType, int64 NbItem)
{
	ScopedCborArchiveEndianness ScopedArchiveEndianness(*Stream, Endianness);

	check(ContainerType == ECborCode::Array || ContainerType == ECborCode::Map);
	CheckContext(ContainerType);

	FCborHeader Header;

	// if NbItem is negative consider the map indefinite
	if (NbItem < 0)
	{
		Header.Set(ContainerType | ECborCode::Indefinite);
		*Stream << Header;
	}
	else
	{
		Header = WriteUIntValue(ContainerType, *Stream, (uint64) NbItem);
	}
	FCborContext Context;
	Context.Header = Header;
	// Length in context for indefinite container is marked as 0 and count up.
	// Map length in context are marked as twice their number of pairs in finite container and counted down.
	// @see CheckContext
	Context.Length = NbItem < 0 ? 0 : (ContainerType == ECborCode::Map ? NbItem * 2 : NbItem);
	ContextStack.Add(MoveTemp(Context));
}

void FCborWriter::WriteContainerEnd()
{
	ScopedCborArchiveEndianness ScopedArchiveEndianness(*Stream, Endianness);

	check(ContextStack.Top().IsIndefiniteContainer());
	FCborHeader Header(ECborCode::Break);
	*Stream << Header;
	ContextStack.Pop();
}

void FCborWriter::WriteNull()
{
	ScopedCborArchiveEndianness ScopedArchiveEndianness(*Stream, Endianness);

	CheckContext(ECborCode::Prim);
	FCborHeader Header(ECborCode::Prim | ECborCode::Null);
	*Stream << Header;
}

void FCborWriter::WriteValue(uint64 Value)
{
	ScopedCborArchiveEndianness ScopedArchiveEndianness(*Stream, Endianness);

	CheckContext(ECborCode::Uint);
	WriteUIntValue(ECborCode::Uint, *Stream, Value);
}

void FCborWriter::WriteValue(int64 Value)
{
	ScopedCborArchiveEndianness ScopedArchiveEndianness(*Stream, Endianness);

	if (Value < 0)
	{
		CheckContext(ECborCode::Int);
		WriteUIntValue(ECborCode::Int, *Stream, ~Value);
	}
	else
	{
		CheckContext(ECborCode::Uint);
		WriteUIntValue(ECborCode::Uint, *Stream, Value);
	}
}

void FCborWriter::WriteValue(bool Value)
{
	ScopedCborArchiveEndianness ScopedArchiveEndianness(*Stream, Endianness);

	CheckContext(ECborCode::Prim);
	FCborHeader Header(ECborCode::Prim | (Value ? ECborCode::True : ECborCode::False));
	*Stream << Header;
}

void FCborWriter::WriteValue(float Value)
{
	ScopedCborArchiveEndianness ScopedArchiveEndianness(*Stream, Endianness);

	CheckContext(ECborCode::Prim);
	FCborHeader Header(ECborCode::Prim | ECborCode::Value_4Bytes);
	*Stream << Header;
	*Stream << Value;
}

void FCborWriter::WriteValue(double Value)
{
	ScopedCborArchiveEndianness ScopedArchiveEndianness(*Stream, Endianness);

	CheckContext(ECborCode::Prim);
	FCborHeader Header(ECborCode::Prim | ECborCode::Value_8Bytes);
	*Stream << Header;
	*Stream << Value;
}

void FCborWriter::WriteValue(const FString& Value)
{
	WriteValue(FStringView(Value));
}

void FCborWriter::WriteValue(const FAnsiString& Value)
{
	WriteValue(FAnsiStringView(Value));
}

void FCborWriter::WriteValue(const FUtf8String& Value)
{
	WriteValue(FUtf8StringView(Value));
}

void FCborWriter::WriteValue(const FStringView& Value)
{
	ScopedCborArchiveEndianness ScopedArchiveEndianness(*Stream, Endianness);

	CheckContext(ECborCode::TextString);
	const FTCHARToUTF8 UTF8String(Value);
	// Write string header
	WriteUIntValue(ECborCode::TextString, *Stream, (uint64)UTF8String.Length());
	// Write string
	check(sizeof(decltype(*UTF8String.Get())) == 1);
	Stream->Serialize(const_cast<char*>(reinterpret_cast<const char*>(UTF8String.Get())), UTF8String.Length());
}

void FCborWriter::WriteValue(const FUtf8StringView& Value)
{
	ScopedCborArchiveEndianness ScopedArchiveEndianness(*Stream, Endianness);

	CheckContext(ECborCode::TextString);
	// Write string header
	WriteUIntValue(ECborCode::TextString, *Stream, (uint64)Value.Len());
	// Write string
	check(sizeof(decltype(*Value.GetData())) == 1);
	Stream->Serialize(const_cast<char*>(reinterpret_cast<const char*>(Value.GetData())), Value.Len());
}

void FCborWriter::WriteValue(const char* CString, uint64 Length)
{
	ScopedCborArchiveEndianness ScopedArchiveEndianness(*Stream, Endianness);

	CheckContext(ECborCode::ByteString);
	// Write c string header
	WriteUIntValue(ECborCode::ByteString, *Stream, Length);
	Stream->Serialize(const_cast<char*>(CString), Length);
}

void FCborWriter::WriteValue(const uint8* Bytes, uint64 Length)
{
	static_assert(sizeof(uint8) == sizeof(char), "Expected char type to be 1 byte");
	WriteValue(reinterpret_cast<const char*>(Bytes), Length);
}

FCborHeader FCborWriter::WriteUIntValue(FCborHeader Header, FArchive& Ar, uint64 Value)
{
	if (Value < 24)
	{
		Header.Set(Header.MajorType() | (ECborCode)Value);
		Ar << Header;
	}
	else if (Value < 256)
	{
		Header.Set(Header.MajorType() | ECborCode::Value_1Byte);
		Ar << Header;
		uint8 Temp = (uint8)(Value);
		Ar << Temp;
	}
	else if (Value < 65536)
	{
		Header.Set((uint8)(Header.MajorType() | ECborCode::Value_2Bytes));
		Ar << Header;
		uint16 Temp = (uint16)Value;
		Ar << Temp;
	}
	else if (Value < 0x100000000L)
	{
		Header.Set((uint8)(Header.MajorType() | ECborCode::Value_4Bytes));
		Ar << Header;
		uint32 Temp = (uint32)Value;
		Ar << Temp;
	}
	else
	{
		Header.Set((uint8)(Header.MajorType() | ECborCode::Value_8Bytes));
		Ar << Header;
		uint64 Temp = Value;
		Ar << Temp;
	}
	return Header;
}

void FCborWriter::CheckContext(ECborCode MajorType)
{
	FCborContext& Context = ContextStack.Top();
	if (Context.IsIndefiniteContainer())
	{
		++Context.Length;
		check(!Context.IsString() || MajorType != Context.MajorType());
	}
	else if (Context.IsFiniteContainer())
	{
		if (--Context.Length == 0)
		{
			ContextStack.Pop();
		}
	}
}
