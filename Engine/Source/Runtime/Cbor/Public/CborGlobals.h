// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

CBOR_API DECLARE_LOG_CATEGORY_EXTERN(LogCbor, Log, All);
