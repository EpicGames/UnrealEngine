// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CborTypes.h"
#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"

class FArchive;

/**
 * Reader for a the cbor protocol encoded stream
 * @see http://cbor.io
 */
class FCborReader
{
public:
	/**
	 * Construct a CBOR reader.
	 * @param InStream The stream containing the CBOR data.
	 * @param InReaderEndianness Specify which endianness should be use to read the archive.
	 * @note CBOR standard endianness is big endian. For interoperability with external tools, the standard endianness should be used. For internal usage, the platform endianness is faster.
	 */
	CBOR_API FCborReader(FArchive* InStream, ECborEndianness InReaderEndianness = ECborEndianness::Platform);
	CBOR_API ~FCborReader();

	/** @return the archive we are reading from. */
	CBOR_API const FArchive* GetArchive() const;

	/** @return true if the reader is in error. */
	CBOR_API bool IsError() const;

	/** @return A cbor Header containing an error code as its raw code. */
	CBOR_API FCborHeader GetError() const;

	/**
	 * The cbor context of the reader can either be
	 * a container context or a dummy.
	 * A reference to the context shouldn't be held while calling ReadNext.
	 * @return The current cbor context. */
	CBOR_API const FCborContext& GetContext() const;

	/**
	 * Read the next value from the cbor stream.
	 * @param OutContext the context to read the value into.
	 * @return true if successful, false if an error was returned or the end of the stream was reached.
	 */
	CBOR_API bool ReadNext(FCborContext& OutContext);

	/**
	 * Skip a container of ContainerType type
	 * @param ContainerType the container we expect to skip.
	 * @return true if successful, false if the current container wasn't a ContainerType or an error occurred.
	 */
	CBOR_API bool SkipContainer(ECborCode ContainerType);
	
private:
	/** Read a uint value from Ar into OutContext and also return it. */
	static uint64 ReadUIntValue(FCborContext& OutContext, FArchive& Ar);
	/** Read a Prim value from Ar into OutContext. */
	static void ReadPrimValue(FCborContext& OutContext, FArchive& Ar);

	/** Set an error in the reader and return it. */
	FCborHeader SetError(ECborCode ErrorCode);

	/** The archive we are reading from. */
	FArchive* Stream;
	/** Holds the context stack for the reader. */
	TArray<FCborContext> ContextStack;
	/** Read the CBOR data using the specified endianness. */
	ECborEndianness Endianness;
};
