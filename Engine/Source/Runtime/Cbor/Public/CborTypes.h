// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/AnsiString.h"
#include "Containers/Utf8String.h"
#include "Misc/EnumClassFlags.h"

/** 
 * Possible cbor code for cbor headers.
 * @see http://cbor.io
 */
enum class ECborCode : uint8
{
	None				= 0,		// no code
	// Major Types
	Uint				= 0 << 5,	// positive/unsigned int
	Int					= 1 << 5,	// negative number
	ByteString			= 2 << 5,	// byte string
	TextString			= 3 << 5,	// text string
	Array				= 4 << 5,	// array
	Map					= 5 << 5,	// map
	Tag					= 6 << 5,	// semantic tag
	Prim				= 7 << 5,	// bool, null, char, half-float, float, double, break code

	// Additional Value Info
	Value_1Byte			= 0x18,		// Additional value in next byte
	Value_2Bytes		= 0x19,		// Additional value in next 2 bytes
	Value_4Bytes		= 0x1A,		// Additional value in next 4 bytes
	Value_8Bytes		= 0x1B,		// Additional value in next 8 bytes
	Unused_28			= 0x1C,		// Unused value in protocol
	Unused_29			= 0x1D,		// Unused value in protocol
	Unused_30			= 0x1E,		// Unused value in protocol
	Indefinite			= 0x1F,		// Indicate indefinite containers

	// Prim type codes
	False				= 0x14,		// boolean
	True				= 0x15,		// boolean
	Null				= 0x16,		// null value
	Undefined			= 0x17,		// undefined, unused in the writer

	// Special values
	Break				= 0xFF,		// break code (Prim | 31)

	// Protocol unused values, used to report context or errors
	// State
	Dummy				= 0x1C,		// mark a dummy					(Uint | 28)
	StreamEnd			= 0x3C,		// stream end					(Int | 28)
	// Errors
	ErrorReservedItem	= 0x1D,		// reserved value				(Uint | 29)
	ErrorStreamFailure	= 0x1E,		// stream error					(Uint | 30)
	ErrorBreak			= 0x3D,		// break not allowed			(Int | 29)
	ErrorMapContainer	= 0x3E,		// odd item number in map		(Int | 30)
	ErrorContext		= 0x5E,		// reader/writer context error	(ByteString | 30)
	ErrorStringNesting	= 0x7D,		// infinite string wrong type	(TextString | 29)
	ErrorStringLength	= 0x7E,		// invalid string length		(TextString | 30)
};
ENUM_CLASS_FLAGS(ECborCode);

class FCborReader;
class FCborWriter;

/** 
 * class that represent a cbor header
 */
class FCborHeader
{
public:
	FCborHeader(uint8 InHeader = 0)
		: Header(InHeader)
	{}
	FCborHeader(ECborCode InHeader)
		: Header((uint8)InHeader)
	{}

	/** Set a cbor code for the header. */
	void Set(ECborCode Code)
	{
		Header = (uint8)Code;
	}

	/** Set a cbor code as a uint8. */
	void Set(uint8 Code)
	{
		Header = Code;
	}

	/** Get the cbor header as a uint8 */
	uint8 Raw() const
	{
		return Header;
	}

	/** Get the cbor header raw code. */
	ECborCode RawCode() const
	{
		return (ECborCode)Header;
	}

	/** Get the major type part of the cbor header. */
	ECborCode MajorType() const
	{
		return (ECborCode)(Header & (7 << 5));
	}

	/** Get the additional value part of the cbor header. */
	ECborCode AdditionalValue() const
	{
		return (ECborCode)(Header & 0x1F);
	}

	/** Serialization helper */
	friend FArchive& operator<<(FArchive& Ar, FCborHeader& InHeader)
	{
		return Ar << InHeader.Header;
	}

private:
	/** Hold the header value. */
	uint8 Header;
};

/**
 * class that represent a cbor context
 * which consists of a header and value pair
 */
struct FCborContext
{
	FCborContext()
		: Header(ECborCode::Dummy)
		, IntValue(0)
	{}

	/** Reset the context to a dummy state. */
	void Reset()
	{
		*this = FCborContext();
	}

	/** @return the context header raw code. */
	ECborCode RawCode() const
	{
		return Header.RawCode();
	}

	/** @return the context header major type. */
	ECborCode MajorType() const
	{
		return Header.MajorType();
	}

	/** @return the context header additional value. */
	ECborCode AdditionalValue() const
	{
		return Header.AdditionalValue();
	}

	/** @return true if this is a dummy context. */
	bool IsDummy()
	{
		return Header.RawCode() == ECborCode::Dummy;
	}

	/** @return true if this context represents an error code. */
	bool IsError() const
	{
		// All error code have their additional value set to those 2 protocol unused values.
		return AdditionalValue() == ECborCode::Unused_29 || AdditionalValue() == ECborCode::Unused_30;
	}

	/** @return true if this context represent a break code. */
	bool IsBreak() const
	{
		return Header.RawCode() == ECborCode::Break;
	}

	/** @return true if this context represents a string type. */
	bool IsString() const
	{
		return MajorType() == ECborCode::TextString || MajorType() == ECborCode::ByteString;
	}

	/** @return true if this context represents a container. (indefinite string are containers.)*/
	bool IsContainer() const
	{
		return IsIndefiniteContainer() || IsFiniteContainer();
	}

	/** @return true if this context represents an indefinite container. */
	bool IsIndefiniteContainer() const
	{
		return (MajorType() == ECborCode::Array || MajorType() == ECborCode::Map || MajorType() == ECborCode::ByteString || MajorType() == ECborCode::TextString) 
			&& AdditionalValue() == ECborCode::Indefinite;
	}

	/** @return true if this context represents an finite container. */
	bool IsFiniteContainer() const
	{
		return (MajorType() == ECborCode::Array || MajorType() == ECborCode::Map) 
			&& AdditionalValue() != ECborCode::Indefinite;
	}

	/** @return the context as the container code the break context is associated with. */
	ECborCode AsBreak() const
	{
		check(Header.RawCode() == ECborCode::Break && RawTextValue.Num() == 1);
		return (ECborCode)RawTextValue[0];
	}

	/** @return the context as a container length. Map container returns their length as twice their number of pairs. */
	uint64 AsLength() const
	{
		check(RawCode() == ECborCode::Break || MajorType() == ECborCode::Array || MajorType() == ECborCode::Map || MajorType() == ECborCode::ByteString || MajorType() == ECborCode::TextString);
		return Length;
	}

	/** @return the context as an unsigned int. */
	uint64 AsUInt() const
	{
		check(MajorType() == ECborCode::Uint);
		return UIntValue;
	}

	/** @return the context as an int. */
	int64 AsInt() const
	{
		check(MajorType() == ECborCode::Int || MajorType() == ECborCode::Uint);
		return IntValue;
	}

	/** @return the context as a bool. */
	bool AsBool() const
	{
		check(MajorType() == ECborCode::Prim && (AdditionalValue() == ECborCode::False || AdditionalValue() == ECborCode::True));
		return BoolValue;
	}

	/** @return the context as a float. */
	float AsFloat() const
	{
		check(Header.RawCode() == (ECborCode::Prim | ECborCode::Value_4Bytes));
		return FloatValue;
	}

	/** @return the context as a double. */
	double AsDouble() const
	{
		check(Header.RawCode() == (ECborCode::Prim | ECborCode::Value_8Bytes));
		return DoubleValue;
	}

	/** @return the context as a string. */
	FString AsString() const
	{
		check(MajorType() == ECborCode::TextString);
		return FString(FUTF8ToTCHAR(RawTextValue.GetData()).Get());
	}

	/** @return the context as a string. */
	FAnsiString AsAnsiString() const
	{
		// Data loss is possible here
		check(MajorType() == ECborCode::TextString);
		return FAnsiString(RawTextValue.GetData());
	}

	/** @return the context as a string. */
	FUtf8String AsUtf8String() const
	{
		check(MajorType() == ECborCode::TextString);
		return FUtf8String(RawTextValue.GetData());
	}

	/** @return the context as a C string. */
	const char* AsCString() const
	{
		check(MajorType() == ECborCode::ByteString);
		return RawTextValue.GetData();
	}

	/** @return the context as a raw byte array. */
	TArrayView<const uint8> AsByteArray() const
	{
		check(MajorType() == ECborCode::ByteString);
		return MakeArrayView(reinterpret_cast<const uint8*>(RawTextValue.GetData()), RawTextValue.Num() - 1); // -1 to exclude the null terminated added by the reader.
	}

private:
	friend class FCborReader;
	friend class FCborWriter;

	FCborContext(ECborCode Code)
		: Header(Code)
		, IntValue(0)
	{}

	// Holds the context header.
	FCborHeader Header;

	/** Union to hold the context value. */
	union
	{
		int64	IntValue;
		uint64	UIntValue;
		bool	BoolValue;
		float	FloatValue;
		double	DoubleValue;
		uint64	Length;
	};
	// Hold text value separately since, non trivial type are a mess in union, also used to report container type for break code
	TArray<char> RawTextValue;
};

/** Defines in which endianness the CBOR data must be written. The official endiannes is 'big endian' but Unreal use both. */
enum class ECborEndianness
{
	/** Read or write the CBOR using the current platform endianness. */
	Platform,

	/** Read or write the CBOR data in big endian (CBOR standard compliant) */
	BigEndian,

	/** Read or write the CBOR data in little endian (Fine for internal usage, but not usable with CBOR standard tools) */
	LittleEndian,

	/** The endianness complying to the CBOR specifications. */
	StandardCompliant = BigEndian,
};

/**
 * Ensure an archive is swapping the bytes according to which endianness was requested to read or write the CBOR.
 * @node The official CBOR endianness is 'big endian', but Unreal uses boths for backward compability reasons.
 */
class ScopedCborArchiveEndianness
{
public:
	/** Set the archive to read or write the data according to the selected endianness. */
	ScopedCborArchiveEndianness(FArchive& InArchive, ECborEndianness Endianness) : Archive(InArchive), bOldByteSwappingState(InArchive.IsByteSwapping())
	{
		constexpr bool bLittleEndianPlatform = PLATFORM_LITTLE_ENDIAN != 0;
		Archive.SetByteSwapping(Endianness != ECborEndianness::Platform && ((Endianness == ECborEndianness::BigEndian && bLittleEndianPlatform) || (Endianness == ECborEndianness::LittleEndian && !bLittleEndianPlatform)));
	}

	~ScopedCborArchiveEndianness()
	{
		Archive.SetByteSwapping(bOldByteSwappingState); // Restore the original state whatever it was.
	}

private:
	FArchive& Archive;
	bool bOldByteSwappingState;
};
