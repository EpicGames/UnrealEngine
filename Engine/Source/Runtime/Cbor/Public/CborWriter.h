// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CborTypes.h"
#include "Containers/ContainersFwd.h"
#include "Containers/Array.h"
#include "Containers/StringFwd.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"

class FArchive;

/**
* Writer for encoding a stream with the cbor protocol
* @see http://cbor.io
*/
class FCborWriter
{
public:
	/*
	 * Construct a CBOR writer.
	 * @param InStream The stream used to write the CBOR data.
	 * @param InWriterEndianness Specify which endianness should be use to write the archive.
	 * @note CBOR standard endianness is big endian. For interoperability with external tools, the standard endianness should be used. For internal usage, the platform endianness is faster.
	 */
	CBOR_API FCborWriter(FArchive* InStream, ECborEndianness InWriterEndianness = ECborEndianness::Platform);
	CBOR_API ~FCborWriter();

public:
	/** @return the archive we are writing to. */
	CBOR_API const FArchive* GetArchive() const;

	/**
	 * Write a container start code.
	 * @param ContainerType container major type, either array or map.
	 * @param NbItem the number of item in the container or negative to indicate indefinite containers.
	 */
	CBOR_API void WriteContainerStart(ECborCode ContainerType, int64 NbItem);

	/** Write a container break code, need a indefinite container context. */
	CBOR_API void WriteContainerEnd();

	/** Write a value. */

	CBOR_API void WriteNull();
	CBOR_API void WriteValue(uint64 Value);
	CBOR_API void WriteValue(int64 Value);
	CBOR_API void WriteValue(bool Value);
	CBOR_API void WriteValue(float Value);
	CBOR_API void WriteValue(double Value);
	CBOR_API void WriteValue(const FString& Value);
	CBOR_API void WriteValue(const FAnsiString& Value);
	CBOR_API void WriteValue(const FUtf8String& Value);
	CBOR_API void WriteValue(const FStringView& Value);
	CBOR_API void WriteValue(const FUtf8StringView& Value);
	CBOR_API void WriteValue(const char* CString, uint64 Length);
	CBOR_API void WriteValue(const uint8* Bytes, uint64 Length);

private:
	/** Write a uint Value for Header in Ar and return the final generated cbor Header. */
	static FCborHeader WriteUIntValue(FCborHeader Header, FArchive& Ar, uint64 Value);

	/** Validate the current writer context for MajorType. */
	void CheckContext(ECborCode MajorType);

	/** The archive being written to. */
	FArchive* Stream;
	/** The writer context stack. */
	TArray<FCborContext> ContextStack;
	/** Write the CBOR data using the specified endianness. */
	ECborEndianness Endianness;
};

