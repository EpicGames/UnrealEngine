// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_LOW_LEVEL_TESTS

#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"

#include "CborReader.h"
#include "CborWriter.h"
#include "CoreGlobals.h"
#include "TestHarness.h"

GROUP_TEST_CASE("CBor", "CBor::FCbor::Smoke Test", "[Smoke]")
{
	// Run the test twices, once with little-endian encoding, once with big-endian encoding.
	auto RunWithEndiannessFn = [](ECborEndianness Endianness)
	{
		// Create the Writer
		TArray<uint8> Bytes;
		TUniquePtr<FArchive> OutputStream = MakeUnique<FMemoryWriter>(Bytes);
		FCborWriter Writer(OutputStream.Get(), Endianness);

		// Create the Reader
		TUniquePtr<FArchive> InputStream = MakeUnique<FMemoryReader>(Bytes);
		FCborReader Reader(InputStream.Get(), Endianness);

		int64 TestInt = 0;
		FCborContext Context;

		// Positive Integer Item
		Writer.WriteValue(TestInt);
		CHECK(Reader.ReadNext(Context) == true);
		CHECK(Context.MajorType() == ECborCode::Uint);
		CHECK(Context.AsUInt() == TestInt);
		CHECK(Context.AsInt() == TestInt);
	
		TestInt = 1;
		Writer.WriteValue(TestInt);
		CHECK(Reader.ReadNext(Context) == true);
		CHECK(Context.MajorType() == ECborCode::Uint);
		CHECK(Context.AsUInt() == TestInt);
		CHECK(Context.AsInt() == TestInt);

		TestInt = 10;
		Writer.WriteValue(TestInt);
		CHECK(Reader.ReadNext(Context) == true);
		CHECK(Context.MajorType() == ECborCode::Uint);
		CHECK(Context.AsUInt() == TestInt);
		CHECK(Context.AsInt() == TestInt);

		TestInt = 23;
		Writer.WriteValue(TestInt);
		CHECK(Reader.ReadNext(Context) == true);
		CHECK(Context.MajorType() == ECborCode::Uint);
		CHECK(Context.AsUInt() == TestInt);
		CHECK(Context.AsInt() == TestInt);

		TestInt = 24;
		Writer.WriteValue(TestInt);
		CHECK(Reader.ReadNext(Context) == true);
		CHECK(Context.MajorType() == ECborCode::Uint);
		CHECK(Context.AdditionalValue() == ECborCode::Value_1Byte);
		CHECK(Context.AsUInt() == TestInt);
		CHECK(Context.AsInt() == TestInt);

		TestInt = 1000;
		Writer.WriteValue(TestInt);
		CHECK(Reader.ReadNext(Context) == true);
		CHECK(Context.MajorType() == ECborCode::Uint);
		CHECK(Context.AdditionalValue() == ECborCode::Value_2Bytes);

		CHECK(Context.AsUInt() == TestInt);
		CHECK(Context.AsInt() == TestInt);

		TestInt = 3000000000;
		Writer.WriteValue(TestInt);
		CHECK(Reader.ReadNext(Context) == true);
		CHECK(Context.MajorType() == ECborCode::Uint);
		CHECK(Context.AdditionalValue() == ECborCode::Value_4Bytes);

		CHECK(Context.AsUInt() == TestInt);
		CHECK(Context.AsInt() == TestInt);

		TestInt = 9223372036854775807;
		Writer.WriteValue(TestInt);
		CHECK(Reader.ReadNext(Context) == true);
		CHECK(Context.MajorType() == ECborCode::Uint);
		CHECK(Context.AdditionalValue() == ECborCode::Value_8Bytes);
		CHECK(Context.AsUInt() == TestInt);
		CHECK(Context.AsInt() == TestInt);

		// Negative numbers

		TestInt = -1;
		Writer.WriteValue(TestInt);
		CHECK(Reader.ReadNext(Context) == true);
		CHECK(Context.MajorType() == ECborCode::Int);
		CHECK(Context.AsInt() == TestInt);

		TestInt = -23;
		Writer.WriteValue(TestInt);
		CHECK(Reader.ReadNext(Context) == true);
		CHECK(Context.MajorType() == ECborCode::Int);
		CHECK(Context.AsInt() == TestInt);

		TestInt = -25;
		Writer.WriteValue(TestInt);
		CHECK(Reader.ReadNext(Context) == true);
		CHECK(Context.MajorType() == ECborCode::Int);
		CHECK(Context.AdditionalValue() == ECborCode::Value_1Byte);
		CHECK(Context.AsInt() == TestInt);

		TestInt = -1000;
		Writer.WriteValue(TestInt);
		CHECK(Reader.ReadNext(Context) == true);
		CHECK(Context.MajorType() == ECborCode::Int);
		CHECK(Context.AdditionalValue() == ECborCode::Value_2Bytes);
		CHECK(Context.AsInt() == TestInt);

		TestInt = -3000000000LL;
		Writer.WriteValue(TestInt);
		CHECK(Reader.ReadNext(Context) == true);
		CHECK(Context.MajorType() == ECborCode::Int);
		CHECK(Context.AdditionalValue() == ECborCode::Value_4Bytes);
		CHECK(Context.AsInt() == TestInt);

		TestInt = -92233720368547758LL; //-9223372036854775807LL;
		Writer.WriteValue(TestInt);
		CHECK(Reader.ReadNext(Context) == true);
		CHECK(Context.MajorType() == ECborCode::Int);
		CHECK(Context.AdditionalValue() == ECborCode::Value_8Bytes);
		CHECK(Context.AsInt() == TestInt);

		// Bool

		bool TestBool = false;
		Writer.WriteValue(TestBool);
		CHECK(Reader.ReadNext(Context) == true);
		CHECK(Context.MajorType() == ECborCode::Prim);
		CHECK(Context.AdditionalValue() == ECborCode::False);
		CHECK(Context.AsBool() == TestBool);

		TestBool = true;
		Writer.WriteValue(TestBool);
		CHECK(Reader.ReadNext(Context) == true);
		CHECK(Context.MajorType() == ECborCode::Prim);
		CHECK(Context.AdditionalValue() == ECborCode::True);
		CHECK(Context.AsBool() == TestBool);

		// Float

		float TestFloat = 3.14159265f;
		Writer.WriteValue(TestFloat);
		CHECK(Reader.ReadNext(Context) == true);
		CHECK(Context.MajorType() == ECborCode::Prim);
		CHECK(Context.AdditionalValue() == ECborCode::Value_4Bytes);
		CHECK(Context.AsFloat() == TestFloat);

		// Double

		double TestDouble = 3.14159265; // 3.4028234663852886e+38;
		Writer.WriteValue(TestDouble);
		CHECK(Reader.ReadNext(Context) == true);
		CHECK(Context.MajorType() == ECborCode::Prim);
		CHECK(Context.AdditionalValue() == ECborCode::Value_8Bytes);
		CHECK(Context.AsDouble() == TestDouble);

		// String

		FString TestString("ANSIString");

		Writer.WriteValue(TestString);
		CHECK(Reader.ReadNext(Context) == true);
		CHECK(Context.MajorType() == ECborCode::TextString);
		CHECK(Context.AsString() == TestString);

		TestString = TEXT("\u3042\u308A\u304C\u3068\u3046");
		Writer.WriteValue(TestString);
		CHECK(Reader.ReadNext(Context) == true);
		CHECK(Context.MajorType() == ECborCode::TextString);
		CHECK(Context.AsString() == TestString);

		// C String
		char TestCString[] = "Potato";

		Writer.WriteValue(TestCString, (sizeof(TestCString) / sizeof(char)) - 1); // do not count the null terminating character
		CHECK(Reader.ReadNext(Context) == true);
		CHECK(Context.MajorType() == ECborCode::ByteString);
		CHECK(TCString<char>::Strcmp(Context.AsCString(), TestCString) == 0);

		// Byte String. (with '\0' in the middle)
		uint8 ByteString[] = {static_cast<uint8>(-1), static_cast<uint8>(-55), static_cast<uint8>(-128), 0, 1, 15, 127};
		Writer.WriteValue(ByteString, sizeof(ByteString)/sizeof(uint8));
		CHECK(Reader.ReadNext(Context) == true);
		CHECK(Context.MajorType() == ECborCode::ByteString);
		CHECK(FMemory::Memcmp(ByteString, Context.AsByteArray().GetData(), sizeof(ByteString)/sizeof(uint8)) == 0);
		CHECK(Context.AsByteArray().Num() == sizeof(ByteString)/sizeof(uint8));

		// Array
		TArray<int64> IntArray { 0, 1, -1, 10, -1000, -3000000000LL, 240, -24 };
		Writer.WriteContainerStart(ECborCode::Array, IntArray.Num());
		for (int64 Val : IntArray)
		{
			Writer.WriteValue(Val);
		}
		// Array start & length
		CHECK(Reader.ReadNext(Context) == true);
		CHECK(Context.MajorType() == ECborCode::Array);
		CHECK(Context.AsLength() == IntArray.Num());

		for (int64 Val : IntArray)
		{
			CHECK(Reader.ReadNext(Context) == true);
			CHECK(Context.AsInt() == Val);
		}

		// Read array end, report length 0 on finite container
		// although the array wasn't written as indefinite,
		// the reader will emit a virtual break token to notify the container end
		CHECK(Reader.ReadNext(Context) == true);
		CHECK(Context.IsBreak());
		CHECK(Context.AsLength() == 0);

		// Indefinite Array
		Writer.WriteContainerStart(ECborCode::Array, -1);
		for (int64 Val : IntArray)
		{
			Writer.WriteValue(Val);
		}
		Writer.WriteContainerEnd();

		// Array start & length
		CHECK(Reader.ReadNext(Context) == true);
		CHECK(Context.MajorType() == ECborCode::Array);
		CHECK(Context.IsIndefiniteContainer());
		CHECK(Context.AsLength() == 0);

		for (int64 Val : IntArray)
		{
			CHECK(Reader.ReadNext(Context) == true);
			CHECK(Context.AsInt() == Val);
		}

		// Read array end, report length 
		// although the array wasn't written as indefinite,
		// the reader will emit a virtual break token to notify the container end
		CHECK(Reader.ReadNext(Context) == true);
		CHECK(Context.IsBreak());
		CHECK(Context.AsLength() == IntArray.Num());

		// Map
		TMap<FString, FString> StringMap = { {TEXT("Apple"), TEXT("Orange")}, {TEXT("Potato"), TEXT("Tomato")}, {TEXT("Meat"), TEXT("Treat")} };
		Writer.WriteContainerStart(ECborCode::Map, StringMap.Num());

		for (const auto& Pair : StringMap)
		{
			Writer.WriteValue(Pair.Key);
			Writer.WriteValue(Pair.Value);
		}

		// Map start & length
		CHECK(Reader.ReadNext(Context) == true);
		CHECK(Context.MajorType() == ECborCode::Map);
		CHECK(Context.AsLength() == StringMap.Num() * 2);

		for (const auto& Pair : StringMap)
		{
			CHECK(Reader.ReadNext(Context) == true);
			CHECK(Context.AsString() == Pair.Key);
			CHECK(Reader.ReadNext(Context) == true);
			CHECK(Context.AsString() == Pair.Value);
		}

		// Read map end 
		// although the array wasn't written as indefinite,
		// the reader will emit a virtual break token to notify the container end
		CHECK(Reader.ReadNext(Context) == true);
		CHECK(Context.IsBreak());

		CHECK(Reader.ReadNext(Context) == false);
		CHECK(Context.RawCode() == ECborCode::StreamEnd);
		return true;
	};

	RunWithEndiannessFn(ECborEndianness::LittleEndian);
	RunWithEndiannessFn(ECborEndianness::BigEndian);

	// Ensure that setting the endianness does something.
	{
		// Create the big endian writer
		TArray<uint8> BytesBE;
		TUniquePtr<FArchive> OutputStreamBE = MakeUnique<FMemoryWriter>(BytesBE);
		FCborWriter WriterBE(OutputStreamBE.Get(), ECborEndianness::BigEndian);

		// Create the little endian writer
		TArray<uint8> BytesLE;
		TUniquePtr<FArchive> OutputStreamLE = MakeUnique<FMemoryWriter>(BytesLE);
		FCborWriter WriterLE(OutputStreamLE.Get(), ECborEndianness::LittleEndian);

		// Write the same values to both streams and ensure the resulting streams are not identical.
		WriterBE.WriteValue(0x1122334455667788ull);
		WriterLE.WriteValue(0x1122334455667788ull);
		CHECK(BytesBE != BytesLE);
	}

	// Ensure the 'Platform' endianness is correctly handled.
	{
		// Create the writer using the platform endianness.
		TArray<uint8> BytesPlatform;
		TUniquePtr<FArchive> OutputStreamPlatform = MakeUnique<FMemoryWriter>(BytesPlatform);
		FCborWriter WriterPlatform(OutputStreamPlatform.Get(), ECborEndianness::Platform);

		// Create the writer using the same endianness as the platform is expected to deduce to.
		TArray<uint8> Bytes;
		TUniquePtr<FArchive> OutputStream = MakeUnique<FMemoryWriter>(Bytes);
		FCborWriter Writer(OutputStream.Get(), PLATFORM_LITTLE_ENDIAN != 0 ? ECborEndianness::LittleEndian : ECborEndianness::BigEndian);

		// Write the same values to both streams and ensure the resulting streams are identical.
		WriterPlatform.WriteValue((int64)0xDEADBEEFDEADBEEF);
		Writer.WriteValue((int64)0xDEADBEEFDEADBEEF);
		CHECK(BytesPlatform == Bytes);
	}
}

GROUP_TEST_CASE("CBor", "CBor::FCbor::ByteSwap Performance Test", "[.][Perf]")
{
	//  NOTE: The most expensive value to write in CBOR is the double as it needs to swap 8 bytes all the time. Integers can be encoded in 1, 2, 4 or 8 bytes depending on their value, but not double.
	//        String are not considered because they are written in UTF8. The test ensure this is not significantly longer to write/read while swapping the bytes.

	const int32 ReservedByteCount = 512 * 1024 * 1024; // Seems like 512 MB is big enough to draw a worst case scenario. If 512 MB doesn't show a significant performance cost, then real life case should not either.
	const int32 WriteCount = ReservedByteCount / (sizeof(double) + sizeof(FCborHeader)); // Each double has its own header.
	
	auto RunSample = [ReservedByteCount, WriteCount](ECborEndianness Endianness)
	{
		TArray<uint8> Bytes;
		Bytes.Reserve(ReservedByteCount);
		TUniquePtr<FArchive> OutputStream = MakeUnique<FMemoryWriter>(Bytes);
		FCborWriter Writer(OutputStream.Get(), Endianness);

		double Value = 1.0;

		FDateTime WriteStartTime = FDateTime::UtcNow();
		for (int32 I = 0; I < WriteCount; ++I)
		{
			Writer.WriteValue(Value);
			Value += I + I * 0.5;
		}
		FTimespan WriteSpan = FDateTime::UtcNow() - WriteStartTime;
		CHECK(Bytes.Num() <= ReservedByteCount);

		TUniquePtr<FArchive> InputStream = MakeUnique<FMemoryReader>(Bytes);
		FCborReader Reader(InputStream.Get(), Endianness);
		FCborContext Ctx;
		FDateTime ReadStartTime = FDateTime::UtcNow();
		while (Reader.ReadNext(Ctx))
		{
			; // Just consume.
		}
		FTimespan ReadSpan = FDateTime::UtcNow() - ReadStartTime;

		return MakeTuple(WriteSpan, ReadSpan);
	};

	ECborEndianness PlatformEndianness = PLATFORM_LITTLE_ENDIAN ? ECborEndianness::LittleEndian : ECborEndianness::BigEndian;
	ECborEndianness SwapEndianness = PLATFORM_LITTLE_ENDIAN ? ECborEndianness::BigEndian : ECborEndianness::LittleEndian;

	TTuple<FTimespan, FTimespan> PlatformEndianessTimes1 = RunSample(PlatformEndianness);
	TTuple<FTimespan, FTimespan> SwapEndianessTimes1 = RunSample(SwapEndianness);

	TTuple<FTimespan, FTimespan> PlatformEndianessTimes2 = RunSample(PlatformEndianness);
	TTuple<FTimespan, FTimespan> SwapEndianessTimes2 = RunSample(SwapEndianness);

	TTuple<FTimespan, FTimespan> PlatformEndianessTimes3 = RunSample(PlatformEndianness);
	TTuple<FTimespan, FTimespan> SwapEndianessTimes3 = RunSample(SwapEndianness);

	// Average the times.
	FTimespan AvgWritePlatformEndianess = (PlatformEndianessTimes1.Get<0>() + PlatformEndianessTimes2.Get<0>() + PlatformEndianessTimes3.Get<0>()) / 3;
	FTimespan AvgReadPlatformEndianess  = (PlatformEndianessTimes1.Get<1>() + PlatformEndianessTimes2.Get<1>() + PlatformEndianessTimes3.Get<1>()) / 3;
	FTimespan AvgWriteSwapEndianess = (SwapEndianessTimes1.Get<0>() + SwapEndianessTimes2.Get<0>() + SwapEndianessTimes3.Get<0>()) / 3;
	FTimespan AvgReadSwapEndianess  = (SwapEndianessTimes1.Get<1>() + SwapEndianessTimes2.Get<1>() + SwapEndianessTimes3.Get<1>()) / 3;

	// Ratio.
	double WriteRatio = static_cast<double>(AvgWriteSwapEndianess.GetTicks()) / static_cast<double>(AvgWritePlatformEndianess.GetTicks());
	double ReadRatio = static_cast<double>(AvgReadSwapEndianess.GetTicks()) / static_cast<double>(AvgReadPlatformEndianess.GetTicks());

	// The ratio is usually around 1 +/- 0.08 as we don't measure significant performance change, but to account for the testing machine workload, be safe, use a large enough margin.
	double Margin = 0.5;
	CHECK(( WriteRatio >= 1.0 - Margin && WriteRatio <= 1.0 + Margin)==true);
	CHECK(( ReadRatio  >= 1.0 - Margin && ReadRatio  <= 1.0 + Margin)==true);
}

#endif